[InputPlatformSettings_Windows InputPlatformSettings]
+HardwareDevices=(InputClassName="SteamController",HardwareDeviceIdentifier="SteamController",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=260)