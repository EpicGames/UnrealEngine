// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/IInputInterface.h"
#include "IInputDevice.h"
#include "ISteamControllerPlugin.h"
#include "SteamSharedModule.h"
#include "GameFramework/InputSettings.h"
#include <steam/isteamcontroller.h>
#include <steam/isteaminput.h>
#include <steam/steam_api_common.h>
#include <steam/steamtypes.h>

DEFINE_LOG_CATEGORY_STATIC(LogSteamController, Log, All);

#if WITH_STEAM_CONTROLLER

/** @todo - do something about this define */
#ifndef MAX_STEAM_CONTROLLERS
#define MAX_STEAM_CONTROLLERS 8
#endif

class FSteamController : public IInputDevice
{

public:

	FSteamController(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) :
		InitialButtonRepeatDelay(0.2),
		ButtonRepeatDelay(0.1),
		MessageHandler(InMessageHandler),
		bSteamControllerInitialized(false),
		InputSettings(nullptr)
	{
		GConfig->GetDouble(TEXT("/Script/Engine.InputSettings"), TEXT("InitialButtonRepeatDelay"), InitialButtonRepeatDelay, GInputIni);
		GConfig->GetDouble(TEXT("/Script/Engine.InputSettings"), TEXT("ButtonRepeatDelay"), ButtonRepeatDelay, GInputIni);

		// Initialize the API, so we can start calling SteamController functions
		SteamAPIHandle = FSteamSharedModule::Get().ObtainSteamClientInstanceHandle();

		// [RCL] 2015-01-23 FIXME: move to some other code than constructor so we can handle failures more gracefully
		if (SteamAPIHandle.IsValid() && (SteamInput() != nullptr))
		{
			const bool bManuallyCallRunFrame = false;
			bSteamControllerInitialized = SteamInput()->Init(bManuallyCallRunFrame);

			InputSettings = GetDefault<UInputSettings>();
			if (InputSettings != nullptr)
			{
				TArray<FName> ActionNames;
				InputSettings->GetActionNames(ActionNames);
				for (const FName ActionName : ActionNames)
				{
					ANSICHAR AnsiActionName[NAME_SIZE];
					ActionName.GetPlainANSIString(AnsiActionName);

					ControllerDigitalActionHandle_t DigitalActionHandle = SteamInput()->GetDigitalActionHandle(AnsiActionName);
					if (DigitalActionHandle > 0)
					{
						DigitalActionHandlesMap.Add(ActionName, DigitalActionHandle);

						for (FControllerState& ControllerState : ControllerStates)
						{
							ControllerState.DigitalStatusMap.Add(ActionName, false);
							ControllerState.DigitalRepeatTimeMap.Add(ActionName, 0.0);
						}
					}
				}

				TArray<FName> AxisNames;
				InputSettings->GetAxisNames(AxisNames);
				for (const FName AxisName : AxisNames)
				{
					ANSICHAR AnsiAxisName[NAME_SIZE];
					AxisName.GetPlainANSIString(AnsiAxisName);

					ControllerAnalogActionHandle_t AnalogActionHandle = SteamInput()->GetAnalogActionHandle(AnsiAxisName);
					if (AnalogActionHandle > 0)
					{
						AnalogActionHandlesMap.Add(AxisName, AnalogActionHandle);
						for (FControllerState& ControllerState : ControllerStates)
						{
							ControllerAnalogActionData_t AnalogActionData;
							AnalogActionData.x = 0.0f;
							AnalogActionData.y = 0.0f;
							ControllerState.AnalogStatusMap.Add(AxisName, AnalogActionData);
						}
					}
				}

				for (FInputActionKeyMapping ActionMapping : InputSettings->GetActionMappings())
				{
					if (ActionMapping.Key.IsGamepadKey())
					{
						DigitalNamesToKeysMap.Add(ActionMapping.ActionName, ActionMapping.Key);
					}
				}

				for (FInputAxisKeyMapping AxisMapping : InputSettings->GetAxisMappings())
				{
					if (AxisMapping.Key.IsGamepadKey() || AxisMapping.Key == EKeys::MouseX || AxisMapping.Key == EKeys::MouseY)
					{
						AxisNamesToKeysMap.Add(AxisMapping.AxisName, AxisMapping.Key);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogSteamController, Log, "SteamController is not available");		
		}
	}

	virtual ~FSteamController()
	{
		if (FSteamSharedModule::IsAvailable() && FSteamSharedModule::Get().AreSteamDllsLoaded() && SteamInput() != nullptr)
		{
			SteamInput()->Shutdown();
		}
	}

	virtual void Tick( float DeltaTime ) override
	{
	}

	virtual void SendControllerEvents() override
	{
		if (!bSteamControllerInitialized)
		{
			return;
		}
		double CurrentTime = FPlatformTime::Seconds();
		ControllerHandle_t ControllerHandles[STEAM_CONTROLLER_MAX_COUNT];
		int32 NumControllers = (int32)SteamInput()->GetConnectedControllers(ControllerHandles);
		for (int32 i = 0; i < NumControllers; i++)
		{
			ControllerHandle_t ControllerHandle = ControllerHandles[i];
			FControllerState& ControllerState = ControllerStates[i];

			// Doesn't seem to be a good way to get a non-localized string, so use generic name for scope
			static FName SystemName(TEXT("SteamController"));
			static FString ControllerName(TEXT("SteamController"));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(this, SystemName, i, ControllerName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FPlatformUserId UserId = FPlatformMisc::GetPlatformUserForUserIndex(i);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(i, UserId, DeviceId);

			for (auto It = DigitalActionHandlesMap.CreateConstIterator(); It; ++It)
			{
				FName DigitalActionName = It.Key();
				ControllerDigitalActionData_t DigitalActionData = SteamInput()->GetDigitalActionData(ControllerHandle, It.Value());
				if (ControllerState.DigitalStatusMap[DigitalActionName] == false && DigitalActionData.bState)
				{
 					MessageHandler->OnControllerButtonPressed(DigitalNamesToKeysMap[DigitalActionName].GetFName(), UserId, DeviceId, false);
					ControllerState.DigitalRepeatTimeMap[DigitalActionName] = FPlatformTime::Seconds() + ButtonRepeatDelay;
				}
				else if (ControllerState.DigitalStatusMap[DigitalActionName] == true && !DigitalActionData.bState)
				{
					MessageHandler->OnControllerButtonReleased(DigitalNamesToKeysMap[DigitalActionName].GetFName(), UserId, DeviceId, false);
				}
				else if (ControllerState.DigitalStatusMap[DigitalActionName] == true && DigitalActionData.bState && ControllerState.DigitalRepeatTimeMap[DigitalActionName] <= CurrentTime)
				{
					ControllerState.DigitalRepeatTimeMap[DigitalActionName] += ButtonRepeatDelay;
					MessageHandler->OnControllerButtonPressed(DigitalNamesToKeysMap[DigitalActionName].GetFName(), UserId, DeviceId, true);
				}

				ControllerState.DigitalStatusMap[DigitalActionName] = DigitalActionData.bState;
			}

			for (auto It = AnalogActionHandlesMap.CreateConstIterator(); It; ++It)
			{
				FName AnalogActionName = It.Key();
				ControllerAnalogActionData_t AnalogActionData = SteamInput()->GetAnalogActionData(ControllerHandle, It.Value());
				if (AxisNamesToKeysMap[AnalogActionName] == EKeys::MouseX || AxisNamesToKeysMap[AnalogActionName] == EKeys::MouseY)
				{
					ControllerState.MouseX += AnalogActionData.x;
					ControllerState.MouseY += AnalogActionData.y;
					MessageHandler->OnRawMouseMove(AnalogActionData.x, AnalogActionData.y);
				}
				else if (AxisNamesToKeysMap[AnalogActionName] == EKeys::Gamepad_LeftX || AxisNamesToKeysMap[AnalogActionName] == EKeys::Gamepad_LeftY)
				{
					if (ControllerState.AnalogStatusMap[It.Key()].x != AnalogActionData.x)
					{
						MessageHandler->OnControllerAnalog(EKeys::Gamepad_LeftX.GetFName(), UserId, DeviceId, AnalogActionData.x);
					}
					
					if (ControllerState.AnalogStatusMap[It.Key()].y != AnalogActionData.y)
					{
						MessageHandler->OnControllerAnalog(EKeys::Gamepad_LeftY.GetFName(), UserId, DeviceId, AnalogActionData.y);
					}
				}
				else if (AxisNamesToKeysMap[AnalogActionName] == EKeys::Gamepad_RightX || AxisNamesToKeysMap[AnalogActionName] == EKeys::Gamepad_RightY)
				{
					if (ControllerState.AnalogStatusMap[It.Key()].x != AnalogActionData.x)
					{
						MessageHandler->OnControllerAnalog(EKeys::Gamepad_RightX.GetFName(), UserId, DeviceId, AnalogActionData.x);
					}

					if (ControllerState.AnalogStatusMap[It.Key()].y != AnalogActionData.y)
					{
						MessageHandler->OnControllerAnalog(EKeys::Gamepad_RightY.GetFName(), UserId, DeviceId, AnalogActionData.y);
					}
				}
				else if (AxisNamesToKeysMap[AnalogActionName] == EKeys::Gamepad_LeftTriggerAxis)
				{
					if (ControllerState.AnalogStatusMap[It.Key()].x != AnalogActionData.x)
					{
						MessageHandler->OnControllerAnalog(EKeys::Gamepad_LeftTriggerAxis.GetFName(), UserId, DeviceId, AnalogActionData.x);
					}
				}
				else if (AxisNamesToKeysMap[AnalogActionName] == EKeys::Gamepad_RightTriggerAxis)
				{
					if (ControllerState.AnalogStatusMap[It.Key()].x != AnalogActionData.x)
					{
						MessageHandler->OnControllerAnalog(EKeys::Gamepad_RightTriggerAxis.GetFName(), UserId, DeviceId, AnalogActionData.x);
					}
				}
				ControllerState.AnalogStatusMap[AnalogActionName] = AnalogActionData;
			}
		}
	}

	void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override
	{
		// Skip unless this is the large channels, which we consider to be the only SteamController feedback channels
		if ((ChannelType != FForceFeedbackChannelType::LEFT_LARGE) && (ChannelType != FForceFeedbackChannelType::RIGHT_LARGE))
		{
			return;
		}

		if ((ControllerId >= 0) && (ControllerId < MAX_STEAM_CONTROLLERS))
		{
			FForceFeedbackValues Values;
			if (ChannelType == FForceFeedbackChannelType::LEFT_LARGE)
			{
				Values.LeftLarge = Value;
			}
			else
			{
				Values.RightLarge = Value;
			}
			UpdateVibration(ControllerId, Values);
		}
	}

	void SetChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) override
	{
		if ((ControllerId >= 0) && (ControllerId < MAX_STEAM_CONTROLLERS))
		{
			UpdateVibration(ControllerId, Values);
		}
	}

	void UpdateVibration(int32 ControllerId, const FForceFeedbackValues& ForceFeedbackValues)
	{
		// make sure there is a valid device for this controller
		ISteamInput* const Controller = SteamInput();
		if (Controller == nullptr || IsGamepadAttached() == false)
		{
			return;
		}

		ControllerHandle_t ControllerHandles[STEAM_CONTROLLER_MAX_COUNT];
		int32 NumControllers = (int32)SteamInput()->GetConnectedControllers(ControllerHandles);
		if (ControllerHandles[ControllerId] == 0)
		{
			return;
		}

		// Steam discussion threads indicate that 4ms is the max length of the pulse, so multiply the values that are passed in by that to try and create the sensation
		// of a "stronger" vibration
		if (ForceFeedbackValues.LeftLarge > 0.f)
		{
			Controller->Legacy_TriggerHapticPulse(ControllerHandles[ControllerId], ESteamControllerPad::k_ESteamControllerPad_Left, ForceFeedbackValues.LeftLarge * 4000.0f);
		}

		if (ForceFeedbackValues.RightLarge > 0.f)
		{
			Controller->Legacy_TriggerHapticPulse(ControllerHandles[ControllerId], ESteamControllerPad::k_ESteamControllerPad_Right, ForceFeedbackValues.RightLarge * 4000.0f);
		}
	}

	virtual void SetMessageHandler(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override
	{
		MessageHandler = InMessageHandler;
	}

	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return false;
	}

	virtual bool IsGamepadAttached() const override
	{
		return (SteamAPIHandle.IsValid() && bSteamControllerInitialized);
	}

private:

	struct FControllerState
	{
		/** Analog status for all actions from previous frame, on a -1.0 to 1.0 range */
		TMap<FName, ControllerAnalogActionData_t> AnalogStatusMap;

		/** Button status for all actions from previous frame (pressed down or not) */
		TMap<FName, bool> DigitalStatusMap;

		/** List of times that if a button is still pressed counts as a "repeated press" */
		TMap<FName, double> DigitalRepeatTimeMap;

		/** Values for force feedback on this controller.  We only consider the LEFT_LARGE channel for SteamControllers */
		FForceFeedbackValues VibeValues;

		int32 MouseX;
		int32 MouseY;

		FControllerState() : MouseX(0), MouseY(0) {}
	};

	/** Controller states */
	FControllerState ControllerStates[MAX_STEAM_CONTROLLERS];

	/** Delay before sending a repeat message after a button was first pressed */
	double InitialButtonRepeatDelay;

	/** Delay before sending a repeat message after a button has been pressed for a while */
	double ButtonRepeatDelay;

	/** handler to send all messages to */
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;

	/** SteamAPI initialized **/
	TSharedPtr<class FSteamClientInstanceHandler> SteamAPIHandle;

	/** SteamController initialized **/
	bool bSteamControllerInitialized;

	/** Default input settings */
	const UInputSettings* InputSettings;

	TMap<FName, ControllerDigitalActionHandle_t> DigitalActionHandlesMap;
	TMap<FName, ControllerAnalogActionHandle_t> AnalogActionHandlesMap;
	TMap<FName, FKey> DigitalNamesToKeysMap;
	TMap<FName, FKey> AxisNamesToKeysMap;
};

#endif // WITH_STEAM_CONTROLLER

class FSteamControllerPlugin : public IInputDeviceModule
{
	virtual TSharedPtr< class IInputDevice > CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override
	{
#if WITH_STEAM_CONTROLLER
		return TSharedPtr< class IInputDevice >(new FSteamController(InMessageHandler));
#else
		return nullptr;
#endif
	}
};

IMPLEMENT_MODULE( FSteamControllerPlugin, SteamController)
