// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISteamControllerPlugin.h"

#if WITH_STEAM_CONTROLLER == 1


// Disable crazy warnings that claim that standard C library is "deprecated".
#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable:4996)
#endif

#if USING_CODE_ANALYSIS
 	MSVC_PRAGMA( warning( push ) )
 	MSVC_PRAGMA( warning( disable : ALL_CODE_ANALYSIS_WARNINGS ) )
#endif	// USING_CODE_ANALYSIS

THIRD_PARTY_INCLUDES_START
THIRD_PARTY_INCLUDES_END

#if USING_CODE_ANALYSIS
 	MSVC_PRAGMA( warning( pop ) )
#endif	// USING_CODE_ANALYSIS

#ifdef _MSC_VER
#pragma warning(pop)
#endif

#endif // WITH_STEAM_CONTROLLER

