// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class SteamController : ModuleRules
{
    public SteamController(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateIncludePathModuleNames.Add("TargetPlatform");

        PublicDependencyModuleNames.AddRange(new string[] 
		{
			"InputDevice",
            "InputCore"
		});

        PrivateDependencyModuleNames.AddRange(new string[]
        {
			"Core",
			"CoreUObject",
			"ApplicationCore",
			"Engine",
			"SteamShared"
		});

        AddEngineThirdPartyPrivateStaticDependencies(Target,
            "Steamworks"
        );
    }
}
