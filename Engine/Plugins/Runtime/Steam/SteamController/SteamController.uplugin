{
	"FileVersion" : 3,
	"Version" : 1.1,
	"VersionName" : "1.1",
	"FriendlyName" : "Steam Controller Plugin",
	"Description" : "InputDevice plugin for Steam controller",
	"Category" : "Input Devices",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SteamController",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"Linux"
			],
			"PlatformArchitectureDenyList" :
			[
				"Win64:arm64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "SteamShared",
			"Enabled": true
		}
	]
}